{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Code Editor",
	"Description" : "[EXPERIMENTAL] Allows editing of code from within the Unreal editor",
	"Category" : "Programming",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CodeEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}