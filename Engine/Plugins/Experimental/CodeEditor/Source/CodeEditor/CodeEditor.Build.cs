// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CodeEditor : ModuleRules
	{
		public CodeEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"SlateCore",
					"Slate",
					"AssetTools",
					"EditorFramework",
					"UnrealEd",
					"PropertyEditor",
					"Kismet",  // for FWorkflowCentricApplication
					"InputCore",
					"DirectoryWatcher",
					"LevelEditor",
					"Engine",
					"ToolMenus",
				}
				);
		}
	}
}
