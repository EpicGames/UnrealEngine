// Copyright Epic Games, Inc. All Rights Reserved.

#include "CPPRichTextSyntaxHighlighterTextLayoutMarshaller.h"
#include "WhiteSpaceTextRun.h"

const TCHAR* Keywords[] =
{
	TEXT("alignas"),
	TEXT("alignof"),
	TEXT("and"),
	TEXT("and_eq"),
	TEXT("asm"),
	TEXT("auto"),
	TEXT("bitand"),
	TEXT("bitor"),
	TEXT("bool"),
	TEXT("break"),
	TEXT("case"),
	TEXT("catch"),
	TEXT("char"),
	TEXT("char16_t"),
	TEXT("char32_t"),
	TEXT("class"),
	TEXT("compl"),
	TEXT("concept"),
	TEXT("const"),
	TEXT("constexpr"),
	TEXT("const_cast"),
	TEXT("continue"),
	TEXT("decltype"),
	TEXT("default"),
	TEXT("delete"),
	TEXT("do"),
	TEXT("double"),
	TEXT("dynamic_cast"),
	TEXT("else"),
	TEXT("enum"),
	TEXT("explicit"),
	TEXT("export"),
	TEXT("extern"),
	TEXT("false"),
	TEXT("float"),
	TEXT("for"),
	TEXT("friend"),
	TEXT("goto"),
	TEXT("if"),
	TEXT("inline"),
	TEXT("int"),
	TEXT("long"),
	TEXT("mutable"),
	TEXT("namespace"),
	TEXT("new"),
	TEXT("noexcept"),
	TEXT("not"),
	TEXT("not_eq"),
	TEXT("nullptr"),
	TEXT("operator"),
	TEXT("or"),
	TEXT("or_eq"),
	TEXT("private"),
	TEXT("protected"),
	TEXT("public"),
	TEXT("register"),
	TEXT("reinterpret_cast"),
	TEXT("requires"),
	TEXT("return"),
	TEXT("short"),
	TEXT("signed"),
	TEXT("sizeof"),
	TEXT("static"),
	TEXT("static_assert"),
	TEXT("static_cast"),
	TEXT("struct"),
	TEXT("switch"),
	TEXT("template"),
	TEXT("this"),
	TEXT("thread_local"),
	TEXT("throw"),
	TEXT("true"),
	TEXT("try"),
	TEXT("typedef"),
	TEXT("typeid"),
	TEXT("typename"),
	TEXT("union"),
	TEXT("unsigned"),
	TEXT("using"),
	TEXT("virtual"),
	TEXT("void"),
	TEXT("volatile"),
	TEXT("wchar_t"),
	TEXT("while"),
	TEXT("xor"),
	TEXT("xor_eq"),
};

const TCHAR* Operators[] =
{
	TEXT("/*"),
	TEXT("*/"),
	TEXT("//"),
	TEXT("\""),
	TEXT("\'"),
	TEXT("::"),
	TEXT(":"),
	TEXT("+="),
	TEXT("++"),
	TEXT("+"),
	TEXT("--"),
	TEXT("-="),
	TEXT("-"),
	TEXT("("),
	TEXT(")"),
	TEXT("["),
	TEXT("]"),
	TEXT("."),
	TEXT("->"),
	TEXT("!="),
	TEXT("!"),
	TEXT("&="),
	TEXT("~"),
	TEXT("&"),
	TEXT("*="),
	TEXT("*"),
	TEXT("->"),
	TEXT("/="),
	TEXT("/"),
	TEXT("%="),
	TEXT("%"),
	TEXT("<<="),
	TEXT("<<"),
	TEXT("<="),
	TEXT("<"),
	TEXT(">>="),
	TEXT(">>"),
	TEXT(">="),
	TEXT(">"),
	TEXT("=="),
	TEXT("&&"),
	TEXT("&"),
	TEXT("^="),
	TEXT("^"),
	TEXT("|="),
	TEXT("||"),
	TEXT("|"),
	TEXT("?"),
	TEXT("="),
	TEXT(","),
	TEXT("{"),
	TEXT("}"),
	TEXT(";"),
};

const TCHAR* PreProcessorKeywords[] =
{
	TEXT("#include"),
	TEXT("#define"),
	TEXT("#ifndef"),
	TEXT("#ifdef"),
	TEXT("#if"),
	TEXT("#else"),
	TEXT("#endif"),
	TEXT("#pragma"),
	TEXT("#undef"),
};

TSharedRef< FCPPRichTextSyntaxHighlighterTextLayoutMarshaller > FCPPRichTextSyntaxHighlighterTextLayoutMarshaller::Create(const FSyntaxTextStyle& InSyntaxTextStyle)
{
	TArray<FSyntaxTokenizer::FRule> TokenizerRules;

	// operators
	for(const auto& Operator : Operators)
	{
		TokenizerRules.Emplace(FSyntaxTokenizer::FRule(Operator));
	}	

	// keywords
	for(const auto& Keyword : Keywords)
	{
		TokenizerRules.Emplace(FSyntaxTokenizer::FRule(Keyword));
	}

	// Pre-processor Keywords
	for(const auto& PreProcessorKeyword : PreProcessorKeywords)
	{
		TokenizerRules.Emplace(FSyntaxTokenizer::FRule(PreProcessorKeyword));
	}

	return MakeShareable(new FCPPRichTextSyntaxHighlighterTextLayoutMarshaller(FSyntaxTokenizer::Create(TokenizerRules), InSyntaxTextStyle));
}

FCPPRichTextSyntaxHighlighterTextLayoutMarshaller::~FCPPRichTextSyntaxHighlighterTextLayoutMarshaller()
{

}

void FCPPRichTextSyntaxHighlighterTextLayoutMarshaller::ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines)
{
	enum class EParseState : uint8
	{
		None,
		LookingForString,
		LookingForCharacter,
		LookingForSingleLineComment,
		LookingForMultiLineComment,
	};

	TArray<FTextLayout::FNewLineData> LinesToAdd;
	LinesToAdd.Reserve(TokenizedLines.Num());

	// Parse the tokens, generating the styled runs for each line
	EParseState ParseState = EParseState::None;
	for(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine : TokenizedLines)
	{
		TSharedRef<FString> ModelString = MakeShareable(new FString());
		TArray< TSharedRef< IRun > > Runs;

		if(ParseState == EParseState::LookingForSingleLineComment)
		{
			ParseState = EParseState::None;
		}

		for(const ISyntaxTokenizer::FToken& Token : TokenizedLine.Tokens)
		{
			const FString TokenText = SourceString.Mid(Token.Range.BeginIndex, Token.Range.Len());

			const FTextRange ModelRange(ModelString->Len(), ModelString->Len() + TokenText.Len());
			ModelString->Append(TokenText);

			FRunInfo RunInfo(TEXT("SyntaxHighlight.CPP.Normal"));
			FTextBlockStyle TextBlockStyle = SyntaxTextStyle.NormalTextStyle;

			const bool bIsWhitespace = FString(TokenText).TrimEnd().IsEmpty();
			if(!bIsWhitespace)
			{
				bool bHasMatchedSyntax = false;
				if(Token.Type == ISyntaxTokenizer::ETokenType::Syntax)
				{
					if(ParseState == EParseState::None && TokenText == TEXT("\""))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.String");
						TextBlockStyle = SyntaxTextStyle.StringTextStyle;
						ParseState = EParseState::LookingForString;
						bHasMatchedSyntax = true;
					}
					else if(ParseState == EParseState::LookingForString && TokenText == TEXT("\""))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Normal");
						TextBlockStyle = SyntaxTextStyle.StringTextStyle;
						ParseState = EParseState::None;
					}
					else if(ParseState == EParseState::None && TokenText == TEXT("\'"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.String");
						TextBlockStyle = SyntaxTextStyle.StringTextStyle;
						ParseState = EParseState::LookingForCharacter;
						bHasMatchedSyntax = true;
					}
					else if(ParseState == EParseState::LookingForCharacter && TokenText == TEXT("\'"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Normal");
						TextBlockStyle = SyntaxTextStyle.StringTextStyle;
						ParseState = EParseState::None;
					}
					else if(ParseState == EParseState::None && TokenText.StartsWith(TEXT("#")))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.PreProcessorKeyword");
						TextBlockStyle = SyntaxTextStyle.PreProcessorKeywordTextStyle;
						ParseState = EParseState::None;
					}
					else if(ParseState == EParseState::None && TokenText == TEXT("//"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Comment");
						TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
						ParseState = EParseState::LookingForSingleLineComment;
					}
					else if(ParseState == EParseState::None && TokenText == TEXT("/*"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Comment");
						TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
						ParseState = EParseState::LookingForMultiLineComment;
					}
					else if(ParseState == EParseState::LookingForMultiLineComment && TokenText == TEXT("*/"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Comment");
						TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
						ParseState = EParseState::None;
					}
					else if(ParseState == EParseState::None && TChar<WIDECHAR>::IsAlpha(TokenText[0]))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Keyword");
						TextBlockStyle = SyntaxTextStyle.KeywordTextStyle;
						ParseState = EParseState::None;
					}
					else if(ParseState == EParseState::None && !TChar<WIDECHAR>::IsAlpha(TokenText[0]))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Operator");
						TextBlockStyle = SyntaxTextStyle.OperatorTextStyle;
						ParseState = EParseState::None;
					}
				}
				
				// It's possible that we fail to match a syntax token if we're in a state where it isn't parsed
				// In this case, we treat it as a literal token
				if(Token.Type == ISyntaxTokenizer::ETokenType::Literal || !bHasMatchedSyntax)
				{
					if(ParseState == EParseState::LookingForString)
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.String");
						TextBlockStyle = SyntaxTextStyle.StringTextStyle;
					}
					else if(ParseState == EParseState::LookingForCharacter)
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.String");
						TextBlockStyle = SyntaxTextStyle.StringTextStyle;
					}
					else if(ParseState == EParseState::LookingForSingleLineComment)
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Comment");
						TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
					}
					else if(ParseState == EParseState::LookingForMultiLineComment)
					{
						RunInfo.Name = TEXT("SyntaxHighlight.CPP.Comment");
						TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
					}
				}

				TSharedRef< ISlateRun > Run = FSlateTextRun::Create(RunInfo, ModelString, TextBlockStyle, ModelRange);
				Runs.Add(Run);
			}
			else
			{
				RunInfo.Name = TEXT("SyntaxHighlight.CPP.WhiteSpace");
				TSharedRef< ISlateRun > Run = FWhiteSpaceTextRun::Create(RunInfo, ModelString, TextBlockStyle, ModelRange, 4);
				Runs.Add(Run);
			}
		}

		LinesToAdd.Emplace(MoveTemp(ModelString), MoveTemp(Runs));
	}

	TargetTextLayout.AddLines(LinesToAdd);
}

FCPPRichTextSyntaxHighlighterTextLayoutMarshaller::FCPPRichTextSyntaxHighlighterTextLayoutMarshaller(TSharedPtr< ISyntaxTokenizer > InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle)
	: FSyntaxHighlighterTextLayoutMarshaller(MoveTemp(InTokenizer))
	, SyntaxTextStyle(InSyntaxTextStyle)
{
}
