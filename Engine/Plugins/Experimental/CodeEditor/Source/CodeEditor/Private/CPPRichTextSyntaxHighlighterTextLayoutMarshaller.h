// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CodeEditorStyle.h"
#include "Framework/Text/SyntaxHighlighterTextLayoutMarshaller.h"

class FTextLayout;

/**
 * Get/set the raw text to/from a text layout, and also inject syntax highlighting for our rich-text markup
 */
class FCPPRichTextSyntaxHighlighterTextLayoutMarshaller : public FSyntaxHighlighterTextLayoutMarshaller
{
public:

	struct FSyntaxTextStyle
	{
		FSyntaxTextStyle()
			: NormalTextStyle(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.CPP.Normal"))
			, OperatorTextStyle(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.CPP.Operator"))
			, KeywordTextStyle(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.CPP.Keyword"))
			, StringTextStyle(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.CPP.String"))
			, NumberTextStyle(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.CPP.Number"))
			, CommentTextStyle(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.CPP.Comment"))
			, PreProcessorKeywordTextStyle(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.CPP.PreProcessorKeyword"))
		{
		}

		FSyntaxTextStyle(const FTextBlockStyle& InNormalTextStyle, const FTextBlockStyle& InOperatorTextStyle, const FTextBlockStyle& InKeywordTextStyle, const FTextBlockStyle& InStringTextStyle, const FTextBlockStyle& InNumberTextStyle, const FTextBlockStyle& InCommentTextStyle, const FTextBlockStyle& InPreProcessorKeywordTextStyle)
			: NormalTextStyle(InNormalTextStyle)
			, OperatorTextStyle(InOperatorTextStyle)
			, KeywordTextStyle(InKeywordTextStyle)
			, StringTextStyle(InStringTextStyle)
			, NumberTextStyle(InNumberTextStyle)
			, CommentTextStyle(InCommentTextStyle)
			, PreProcessorKeywordTextStyle(InPreProcessorKeywordTextStyle)
		{
		}

		FTextBlockStyle NormalTextStyle;
		FTextBlockStyle OperatorTextStyle;
		FTextBlockStyle KeywordTextStyle;
		FTextBlockStyle StringTextStyle;
		FTextBlockStyle NumberTextStyle;
		FTextBlockStyle CommentTextStyle;
		FTextBlockStyle PreProcessorKeywordTextStyle;
	};

	static TSharedRef< FCPPRichTextSyntaxHighlighterTextLayoutMarshaller > Create(const FSyntaxTextStyle& InSyntaxTextStyle);

	virtual ~FCPPRichTextSyntaxHighlighterTextLayoutMarshaller();

protected:

	virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) override;

	FCPPRichTextSyntaxHighlighterTextLayoutMarshaller(TSharedPtr< ISyntaxTokenizer > InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);

	/** Styles used to display the text */
	FSyntaxTextStyle SyntaxTextStyle;

	/** String representing tabs */
	FString TabString;
};
