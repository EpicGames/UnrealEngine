// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeAssetTypeActions.h"
#include "CodeProject.h"
#include "CodeProjectEditor.h"


#define LOCTEXT_NAMESPACE "CodeAssetTypeActions"


FText FCodeAssetTypeActions::GetName() const
{
	return LOCTEXT("CodeProjectActionsName", "Code Project");
}

FColor FCodeAssetTypeActions::GetTypeColor() const
{
	return FColor(255, 255, 0);
}

UClass* FCodeAssetTypeActions::GetSupportedClass() const
{
	return UCodeProject::StaticClass();
}

void FCodeAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UCodeProject* CodeProject = Cast<UCodeProject>(*ObjIt))
		{
			TSharedRef<FCodeProjectEditor> NewCodeProjectEditor(new FCodeProjectEditor());
			NewCodeProjectEditor->InitCodeEditor(Mode, EditWithinLevelEditor, CodeProject);
		}
	}
}

uint32 FCodeAssetTypeActions::GetCategories()
{
	return EAssetTypeCategories::Misc;
}

#undef LOCTEXT_NAMESPACE
