// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreDelegates.h"
#include "ToolMenus.h"
#include "CodeEditorStyle.h"
#include "CodeProject.h"
#include "CodeProjectEditor.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

static const FName CodeEditorTabName( TEXT( "CodeEditor" ) );

#define LOCTEXT_NAMESPACE "CodeEditor"

class FCodeEditor : public IModuleInterface
{
private:
	static TSharedRef<SDockTab> SpawnCodeEditorTab(const FSpawnTabArgs& TabArgs)
	{
		TSharedRef<FCodeProjectEditor> NewCodeProjectEditor(new FCodeProjectEditor());
		NewCodeProjectEditor->InitCodeEditor(EToolkitMode::Standalone, TSharedPtr<class IToolkitHost>(), GetMutableDefault<UCodeProject>());

		return FGlobalTabmanager::Get()->GetMajorTabForTabManager(NewCodeProjectEditor->GetTabManager().ToSharedRef()).ToSharedRef();
	}

	static void OpenCodeEditor()
	{
		SpawnCodeEditorTab(FSpawnTabArgs(TSharedPtr<SWindow>(), FTabId()));	
	}

public:

	virtual void OnPostEngineInit()
	{
		if (UToolMenus::IsToolMenuUIEnabled())
		{
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools");
			FToolMenuSection& Section = Menu->FindOrAddSection("FileProject");

			FToolMenuOwnerScoped OwnerScoped(this);
			{
				FToolMenuEntry& MenuEntry = Section.AddMenuEntry(
					"EditSourceCode",
					LOCTEXT("CodeEditorTabTitle", "Edit Source Code"),
					LOCTEXT("CodeEditorTooltipText", "Open the Code Editor tab."),
					FSlateIcon(FCodeEditorStyle::Get().GetStyleSetName(), "CodeEditor.TabIcon"),
					FUIAction
					(
						FExecuteAction::CreateStatic(&FCodeEditor::OpenCodeEditor)
					)
				);
				MenuEntry.InsertPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);
			}
		}
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		FCodeEditorStyle::Initialize();

		// Register a tab spawner so that our tab can be automatically restored from layout files
		FGlobalTabmanager::Get()->RegisterTabSpawner( CodeEditorTabName, FOnSpawnTab::CreateStatic( &FCodeEditor::SpawnCodeEditorTab ) )
				.SetDisplayName( LOCTEXT( "CodeEditorTabTitle", "Edit Source Code" ) )
				.SetTooltipText( LOCTEXT( "CodeEditorTooltipText", "Open the Code Editor tab." ) )
				.SetIcon(FSlateIcon(FCodeEditorStyle::Get().GetStyleSetName(), "CodeEditor.TabIcon"));

		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FCodeEditor::OnPostEngineInit);
	}

	virtual void ShutdownModule() override
	{
		// Unregister the tab spawner
		FGlobalTabmanager::Get()->UnregisterTabSpawner( CodeEditorTabName );

		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
		UToolMenus::UnregisterOwner(this);

		FCodeEditorStyle::Shutdown();
	}
};

IMPLEMENT_MODULE( FCodeEditor, CodeEditor )

#undef LOCTEXT_NAMESPACE
