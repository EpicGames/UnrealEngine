// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeEditorCustomization.h"

UCodeEditorCustomization::UCodeEditorCustomization(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FCodeEditorControlCustomization& UCodeEditorCustomization::GetControl(const FName& ControlCustomizationName)
{
	static FCodeEditorControlCustomization Default;

	return Default;
}

const FCodeEditorTextCustomization& UCodeEditorCustomization::GetText(const FName& TextCustomizationName)
{
	static FCodeEditorTextCustomization Default;

	return Default;
}
