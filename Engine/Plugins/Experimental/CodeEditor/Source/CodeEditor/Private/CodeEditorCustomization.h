// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CodeEditorCustomization.generated.h"

USTRUCT()
struct FCodeEditorTextCustomization
{
	GENERATED_USTRUCT_BODY()

	FCodeEditorTextCustomization()
		: Font("")
		, Color(0.0f, 0.0f, 0.0f, 1.0f)
	{
	}

	UPROPERTY(EditAnywhere, Category=Text)
	FString Font;

	UPROPERTY(EditAnywhere, Category=Text)
	FLinearColor Color;
};

USTRUCT()
struct FCodeEditorControlCustomization
{
	GENERATED_USTRUCT_BODY()

	FCodeEditorControlCustomization()
		: Color(0.0f, 0.0f, 0.0f, 1.0f)
	{
	}

	UPROPERTY(EditAnywhere, Category=Controls)
	FLinearColor Color;
};

UCLASS(Config=Editor)
class UCodeEditorCustomization : public UObject
{
	GENERATED_UCLASS_BODY()

	static const FCodeEditorControlCustomization& GetControl(const FName& ControlCustomizationName);

	static const FCodeEditorTextCustomization& GetText(const FName& TextCustomizationName);

private:
	UPROPERTY(EditAnywhere, EditFixedSize, Category=Controls)
	TArray<FCodeEditorControlCustomization> Controls;

	UPROPERTY(EditAnywhere, EditFixedSize, Category=Text)
	TArray<FCodeEditorTextCustomization> Text;
};
