// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeEditorStyle.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateTypes.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/CoreStyle.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr< FSlateStyleSet > FCodeEditorStyle::StyleSet = nullptr;

// Const icon sizes
static const FVector2D Icon8x8(8.0f, 8.0f);
static const FVector2D Icon9x19(9.0f, 19.0f);
static const FVector2D Icon16x16(16.0f, 16.0f);
static const FVector2D Icon20x20(20.0f, 20.0f);
static const FVector2D Icon22x22(22.0f, 22.0f);
static const FVector2D Icon24x24(24.0f, 24.0f);
static const FVector2D Icon28x28(28.0f, 28.0f);
static const FVector2D Icon27x31(27.0f, 31.0f);
static const FVector2D Icon26x26(26.0f, 26.0f);
static const FVector2D Icon32x32(32.0f, 32.0f);
static const FVector2D Icon40x40(40.0f, 40.0f);
static const FVector2D Icon48x48(48.0f, 48.0f);
static const FVector2D Icon75x82(75.0f, 82.0f);
static const FVector2D Icon360x32(360.0f, 32.0f);
static const FVector2D Icon171x39(171.0f, 39.0f);
static const FVector2D Icon170x50(170.0f, 50.0f);
static const FVector2D Icon267x140(170.0f, 50.0f);

void FCodeEditorStyle::Initialize()
{
	// Only register once
	if( StyleSet.IsValid() )
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("CodeEditor") );

	StyleSet->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/CodeEditor/Resources"));

	// Icons
	{
		StyleSet->Set("CodeEditor.TabIcon", new IMAGE_BRUSH("UI/CodeEditor_16x", Icon16x16));

		StyleSet->Set("CodeEditor.Save", new IMAGE_BRUSH("UI/Save_40x", Icon40x40));
		StyleSet->Set("CodeEditor.Save.Small", new IMAGE_BRUSH("UI/Save_40x", Icon16x16));
		StyleSet->Set("CodeEditor.SaveAll", new IMAGE_BRUSH("UI/SaveAll_40x", Icon40x40));
		StyleSet->Set("CodeEditor.SaveAll.Small", new IMAGE_BRUSH("UI/SaveAll_40x", Icon16x16));
	}

	const FSlateFontInfo Consolas10  = DEFAULT_FONT("Mono", 9);

	const FTextBlockStyle NormalText = FTextBlockStyle()
		.SetFont(Consolas10)
		.SetColorAndOpacity(FLinearColor::White)
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetHighlightShape(BOX_BRUSH("UI/TextBlockHighlightShape", FMargin(3.f / 8.f)));

	// Text editor
	{
		StyleSet->Set("TextEditor.NormalText", NormalText);

		StyleSet->Set("SyntaxHighlight.CPP.Normal", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xffdfd706))));// yellow
		StyleSet->Set("SyntaxHighlight.CPP.Operator", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xffcfcfcf)))); // light grey
		StyleSet->Set("SyntaxHighlight.CPP.Keyword", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xff006ab4)))); // blue
		StyleSet->Set("SyntaxHighlight.CPP.String", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xff9e4a1e)))); // pinkish
		StyleSet->Set("SyntaxHighlight.CPP.Number", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xff6db3a8)))); // cyan
		StyleSet->Set("SyntaxHighlight.CPP.Comment", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xff57a64a)))); // green
		StyleSet->Set("SyntaxHighlight.CPP.PreProcessorKeyword", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xffcfcfcf)))); // light grey

		StyleSet->Set("TextEditor.Border", new BOX_BRUSH("UI/TextEditorBorder", FMargin(4.0f/16.0f), FLinearColor(0.02f,0.02f,0.02f,1)));

		const FEditableTextBoxStyle EditableTextBoxStyle = FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetBackgroundImageNormal( FSlateNoResource() )
			.SetBackgroundImageHovered( FSlateNoResource() )
			.SetBackgroundImageFocused( FSlateNoResource() )
			.SetBackgroundImageReadOnly( FSlateNoResource() );
		
		StyleSet->Set("TextEditor.EditableTextBox", EditableTextBoxStyle);
	}

	// Project editor
	{
		StyleSet->Set("ProjectEditor.Border", new BOX_BRUSH("UI/TextEditorBorder", FMargin(4.0f/16.0f), FLinearColor(0.048f,0.048f,0.048f,1)));

		StyleSet->Set("ProjectEditor.Icon.Project", new IMAGE_BRUSH("UI/FolderClosed", Icon16x16, FLinearColor(0.25f,0.25f,0.25f,1)));
		StyleSet->Set("ProjectEditor.Icon.Folder", new IMAGE_BRUSH("UI/FolderClosed", Icon16x16, FLinearColor(0.25f,0.25f,0.25f,1)));
		StyleSet->Set("ProjectEditor.Icon.File", new IMAGE_BRUSH("UI/GenericFile", Icon16x16));
		StyleSet->Set("ProjectEditor.Icon.GenericFile", new IMAGE_BRUSH("UI/GenericFile", Icon16x16));
	}

	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
}

#undef RootToContentDir

void FCodeEditorStyle::Shutdown()
{
	if( StyleSet.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}

const ISlateStyle& FCodeEditorStyle::Get()
{
	return *( StyleSet.Get() );
}

const FName& FCodeEditorStyle::GetStyleSetName()
{
	return StyleSet->GetStyleSetName();
}
