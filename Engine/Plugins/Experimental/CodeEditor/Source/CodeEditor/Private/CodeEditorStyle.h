// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class ISlateStyle;

class FCodeEditorStyle
{
public:

	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();

	static const FName& GetStyleSetName();

private:

	/** Singleton instances of this style. */
	static TSharedPtr< class FSlateStyleSet > StyleSet;	
};
