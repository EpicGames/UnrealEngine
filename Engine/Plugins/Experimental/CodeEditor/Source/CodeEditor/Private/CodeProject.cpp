// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeProject.h"
#include "Misc/Paths.h"


UCodeProject::UCodeProject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Path = FPaths::GameSourceDir();
}
