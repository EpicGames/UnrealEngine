// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CodeProjectItem.h"
#include "CodeProject.generated.h"

UCLASS()
class UCodeProject : public UCodeProjectItem
{
	GENERATED_UCLASS_BODY()

	// @TODO: This class should probably be mostly config/settings stuff, with a details panel allowing editing somewhere
};
