// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeProjectEditor.h"
#include "CodeProject.h"
#include "CodeEditorStyle.h"
#include "CodeProjectEditorCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "SCodeEditor.h"
#include "SCodeProjectEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "CodeProjectEditorToolbar.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"


#define LOCTEXT_NAMESPACE "CodeEditor"

TWeakPtr<FCodeProjectEditor> FCodeProjectEditor::CodeEditor;

const FName CodeEditorAppName = FName(TEXT("CodeEditorApp"));

namespace CodeEditorModes
{
	// Mode identifiers
	static const FName StandardMode(TEXT("StandardMode"));
};

namespace CodeEditorTabs
{
	// Tab identifiers
	static const FName ProjectViewID(TEXT("ProjectView"));
	static const FName CodeViewID(TEXT("Document"));
};

struct FCodeTabSummoner : public FDocumentTabFactoryForObjects<UCodeProjectItem>
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FOnCreateCodeEditorWidget, TSharedRef<FTabInfo>, UCodeProjectItem*);

public:
	FCodeTabSummoner(TSharedPtr<class FCodeProjectEditor> InCodeProjectEditorPtr, FOnCreateCodeEditorWidget CreateCodeEditorWidgetCallback)
		: FDocumentTabFactoryForObjects<UCodeProjectItem>(CodeEditorTabs::CodeViewID, InCodeProjectEditorPtr)
		, CodeProjectEditorPtr(InCodeProjectEditorPtr)
		, OnCreateCodeEditorWidget(CreateCodeEditorWidgetCallback)
	{
	}

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override
	{
		TSharedRef<SCodeEditor> CodeEditor = StaticCastSharedRef<SCodeEditor>(Tab->GetContent());
	//	InCodeProjectEditorPtr.Pin()->OnCodeEditorFocused(CodeEditor);
	}

	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override
	{
		TSharedRef<SCodeEditor> GraphEditor = StaticCastSharedRef<SCodeEditor>(Tab->GetContent());
	//	GraphEditor->NotifyItemChanged();
	}

	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override
	{
		TSharedRef<SCodeEditor> GraphEditor = StaticCastSharedRef<SCodeEditor>(Tab->GetContent());

	//	UCodeProjectItem* Graph = FTabPayload_UObject::CastChecked<UCodeProjectItem>(Payload);
	//	BlueprintEditorPtr.Pin()->GetBlueprintObj()->LastEditedDocuments.Add(FEditedDocumentInfo(Graph, ViewLocation, ZoomAmount));
	}

protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UCodeProjectItem* DocumentID) const override
	{
		return FText::FromString(DocumentID->Name);
	}

	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UCodeProjectItem* DocumentID) const override
	{
		check(Info.TabInfo.IsValid());
		return OnCreateCodeEditorWidget.Execute(Info.TabInfo.ToSharedRef(), DocumentID);
	}

	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UCodeProjectItem* DocumentID) const override
	{
		return FCodeEditorStyle::Get().GetBrush("ProjectEditor.Icon.File");
	}

/*	virtual TSharedRef<FGenericTabHistory> CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload) override
	{
		return MakeShareable(new FSourceTabHistory(SharedThis(this), Payload));
	}*/

protected:
	TWeakPtr<class FCodeProjectEditor> CodeProjectEditorPtr;
	FOnCreateCodeEditorWidget OnCreateCodeEditorWidget;
};


struct FProjectViewSummoner : public FWorkflowTabFactory
{
public:
	FProjectViewSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(CodeEditorTabs::ProjectViewID, InHostingApp)
	{
		TabLabel = LOCTEXT("ProjectTabLabel", "Project");

		bIsSingleton = true;

		ViewMenuDescription = LOCTEXT("ProjectTabMenu_Description", "Project");
		ViewMenuTooltip = LOCTEXT("ProjectTabMenu_ToolTip", "Shows the project panel");
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		TSharedPtr<FCodeProjectEditor> CodeEditorPtr = StaticCastSharedPtr<FCodeProjectEditor>(HostingApp.Pin());
		return SNew(SCodeProjectEditor, CodeEditorPtr->GetCodeProjectBeingEdited());
	}
};


class FBasicCodeEditorMode : public FApplicationMode
{
public:
	FBasicCodeEditorMode(TSharedPtr<class FCodeProjectEditor> InCodeEditor, FName InModeName);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	// End of FApplicationMode interface

protected:
	TWeakPtr<FCodeProjectEditor> MyCodeEditor;
	FWorkflowAllowedTabSet TabFactories;
};

FBasicCodeEditorMode::FBasicCodeEditorMode(TSharedPtr<class FCodeProjectEditor> InCodeEditor, FName InModeName)
	: FApplicationMode(InModeName)
{
	MyCodeEditor = InCodeEditor;

	TabFactories.RegisterFactory(MakeShareable(new FProjectViewSummoner(InCodeEditor)));

	TabLayout = FTabManager::NewLayout("Standalone_CodeEditor_Layout_v1.2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->SetHideTabWell(true)
					->AddTab(CodeEditorTabs::ProjectViewID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell(false)
					->AddTab(CodeEditorTabs::CodeViewID, ETabState::ClosedTab)
				)
			)
		);

	InCodeEditor->GetToolbarBuilder()->AddEditorToolbar(ToolbarExtender);
}

void FBasicCodeEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FCodeProjectEditor> Editor = MyCodeEditor.Pin();
	
	Editor->RegisterToolbarTab(InTabManager.ToSharedRef());

	Editor->PushTabFactories(TabFactories);
}

FCodeProjectEditor::FCodeProjectEditor()
{
}

void FCodeProjectEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	DocumentManager->SetTabManager(InTabManager);

	FWorkflowCentricApplication::RegisterTabSpawners(InTabManager);
}

void FCodeProjectEditor::RegisterToolbarTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

void FCodeProjectEditor::InitCodeEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UCodeProject* CodeProject)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseOtherEditors(CodeProject, this);
	CodeProjectBeingEdited = CodeProject;

	TSharedPtr<FCodeProjectEditor> ThisPtr(SharedThis(this));
	if(!DocumentManager.IsValid())
	{
		DocumentManager = MakeShareable(new FDocumentTracker);
		DocumentManager->Initialize(ThisPtr);
	}

	CodeEditor = ThisPtr;

	TSharedRef<FDocumentTabFactory> GraphEditorFactory = MakeShareable(new FCodeTabSummoner(ThisPtr, FCodeTabSummoner::FOnCreateCodeEditorWidget::CreateSP(this, &FCodeProjectEditor::CreateCodeEditorWidget)));
	DocumentManager->RegisterDocumentFactory(GraphEditorFactory);

	if (!ToolbarBuilder.IsValid())
	{
		ToolbarBuilder = MakeShareable(new FCodeProjectEditorToolbar(SharedThis(this)));
	}

	FCodeProjectEditorCommands::Register();

	// Initialize the asset editor and spawn nothing (dummy layout)
	InitAssetEditor(Mode, InitToolkitHost, CodeEditorAppName, FTabManager::FLayout::NullLayout, /*bCreateDefaultStandaloneMenu=*/ true, /*bCreateDefaultToolbar=*/ true, CodeProject);

	BindCommands();

	// Create the modes and activate one (which will populate with a real layout)
	AddApplicationMode(
		CodeEditorModes::StandardMode, 
		MakeShareable(new FBasicCodeEditorMode(ThisPtr, CodeEditorModes::StandardMode)));
	SetCurrentMode(CodeEditorModes::StandardMode);

	RegenerateMenusAndToolbars();
}

void FCodeProjectEditor::BindCommands()
{
	ToolkitCommands->MapAction(FCodeProjectEditorCommands::Get().Save,
			FExecuteAction::CreateSP(this, &FCodeProjectEditor::Save_Internal),
			FCanExecuteAction::CreateSP(this, &FCodeProjectEditor::CanSave)
			);

	ToolkitCommands->MapAction(FCodeProjectEditorCommands::Get().SaveAll,
			FExecuteAction::CreateSP(this, &FCodeProjectEditor::SaveAll_Internal),
			FCanExecuteAction::CreateSP(this, &FCodeProjectEditor::CanSaveAll)
			);
}

void FCodeProjectEditor::OpenFileForEditing(UCodeProjectItem* Item)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(Item);
	DocumentManager->OpenDocument(Payload, FDocumentTracker::OpenNewDocument);
}

FName FCodeProjectEditor::GetToolkitFName() const
{
	return FName("CodeEditor");
}

FText FCodeProjectEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Code Editor");
}

FText FCodeProjectEditor::GetToolkitName() const
{
	return LOCTEXT("CodeAppToolkitName", "Code Editor");
}

FText FCodeProjectEditor::GetToolkitToolTipText() const
{
	return LOCTEXT("CodeAppLabel", "Code Editor");
}

FString FCodeProjectEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("CodeEditor");
}

FLinearColor FCodeProjectEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

void FCodeProjectEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CodeProjectBeingEdited);
}

TSharedRef<SWidget> FCodeProjectEditor::CreateCodeEditorWidget(TSharedRef<FTabInfo> TabInfo, UCodeProjectItem* Item)
{
	return SNew(SCodeEditor, Item);
}

void FCodeProjectEditor::Save_Internal()
{
	Save();
}

bool FCodeProjectEditor::Save()
{
	if(DocumentManager.IsValid() && DocumentManager->GetActiveTab().IsValid())
	{
		TSharedRef<SCodeEditor> CodeEditorRef = StaticCastSharedRef<SCodeEditor>(DocumentManager->GetActiveTab()->GetContent());
		return CodeEditorRef->Save();
	}

	return false;
}

bool FCodeProjectEditor::CanSave() const
{
	if(DocumentManager.IsValid() && DocumentManager->GetActiveTab().IsValid())
	{
		TSharedRef<SWidget> Content = DocumentManager->GetActiveTab()->GetContent();
		TSharedRef<SCodeEditor> CodeEditorRef = StaticCastSharedRef<SCodeEditor>(Content);
		return CodeEditorRef->CanSave();
	}

	return false;
}

void FCodeProjectEditor::SaveAll_Internal()
{
	SaveAll();
}

bool FCodeProjectEditor::SaveAll()
{
	bool bResult = true;

	if(DocumentManager.IsValid())
	{
		TArray<TSharedPtr<SDockTab>> AllTabs = DocumentManager->GetAllDocumentTabs();
		for(auto& Tab : AllTabs)
		{
			if(Tab.IsValid())
			{
				TSharedRef<SCodeEditor> CodeEditorRef = StaticCastSharedRef<SCodeEditor>(Tab->GetContent());
				if(!CodeEditorRef->Save())
				{
					bResult = false;
				}
			}
		}
	}

	return bResult;
}

bool FCodeProjectEditor::CanSaveAll() const
{
	return true;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
