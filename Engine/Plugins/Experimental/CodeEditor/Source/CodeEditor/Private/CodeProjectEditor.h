// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class FCodeProjectEditorToolbar;
class FDocumentTracker;
class FTabInfo;
class UCodeProject;
class UCodeProjectItem;

class FCodeProjectEditor : public FWorkflowCentricApplication, public FGCObject
{
public:
	FCodeProjectEditor();

	// IToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	// End of IToolkit interface

	// FAssetEditorToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	// End of FAssetEditorToolkit

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCodeProjectEditor");
	}
	// End of FSerializableObject interface

	static TSharedPtr<FCodeProjectEditor> Get()
	{
		return CodeEditor.Pin();
	}

public:
	/** Initialize the code editor */
	void InitCodeEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UCodeProject* CodeProject);

	/** Try to open a new file for editing */
	void OpenFileForEditing(class UCodeProjectItem* Item);

	/** Get the current project being edited by this code editor */
	UCodeProject* GetCodeProjectBeingEdited() const { return CodeProjectBeingEdited.Get(); }

	TSharedRef<SWidget> CreateCodeEditorWidget(TSharedRef<FTabInfo> TabInfo, UCodeProjectItem* Item);

	void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);

	/** Access the toolbar builder for this editor */
	TSharedPtr<class FCodeProjectEditorToolbar> GetToolbarBuilder() { return ToolbarBuilder; }

	bool Save();

	bool SaveAll();

private:
	void BindCommands();

	void Save_Internal();

	void SaveAll_Internal();

	bool CanSave() const;

	bool CanSaveAll() const;

protected:
	TSharedPtr<FDocumentTracker> DocumentManager;

	/** The code project we are currently editing */
	TWeakObjectPtr<UCodeProject> CodeProjectBeingEdited;

	TSharedPtr<class FCodeProjectEditorToolbar> ToolbarBuilder;

	static TWeakPtr<FCodeProjectEditor> CodeEditor;
};
