// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeProjectEditorCommands.h"
#include "CodeEditorStyle.h"


#define LOCTEXT_NAMESPACE "CodeProjectEditorCommands"


FCodeProjectEditorCommands::FCodeProjectEditorCommands() 
	: TCommands<FCodeProjectEditorCommands>("CodeEditor", LOCTEXT("General", "General"), NAME_None, FCodeEditorStyle::GetStyleSetName())
{
}


void FCodeProjectEditorCommands::RegisterCommands()
{
	UI_COMMAND(Save, "Save", "Save the currently active document.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::S));
	UI_COMMAND(SaveAll, "Save All", "Save all open documents.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::S));
}


#undef LOCTEXT_NAMESPACE
