// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FCodeProjectEditorCommands : public TCommands<FCodeProjectEditorCommands>
{
public:
	FCodeProjectEditorCommands();

	TSharedPtr<FUICommandInfo> Save;
	TSharedPtr<FUICommandInfo> SaveAll;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
