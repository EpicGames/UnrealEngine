// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeProjectEditorToolbar.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "CodeProjectEditorCommands.h"
#include "LevelEditorActions.h"
#include "SourceCodeNavigation.h"


void FCodeProjectEditorToolbar::AddEditorToolbar(TSharedPtr<FExtender> Extender)
{
	check(CodeProjectEditor.IsValid());
	TSharedPtr<FCodeProjectEditor> CodeProjectEditorPtr = CodeProjectEditor.Pin();

	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		CodeProjectEditorPtr->GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP( this, &FCodeProjectEditorToolbar::FillEditorToolbar ) );
}

void FCodeProjectEditorToolbar::FillEditorToolbar(FToolBarBuilder& ToolbarBuilder)
{
	TSharedPtr<FCodeProjectEditor> CodeProjectEditorPtr = CodeProjectEditor.Pin();

	ToolbarBuilder.BeginSection(TEXT("FileManagement"));
	{
		ToolbarBuilder.AddToolBarButton(FCodeProjectEditorCommands::Get().Save);
		ToolbarBuilder.AddToolBarButton(FCodeProjectEditorCommands::Get().SaveAll);
	}
	ToolbarBuilder.EndSection();

	// Only show the compile options on machines with the solution (assuming they can build it)
	if ( FSourceCodeNavigation::IsCompilerAvailable() )
	{
		ToolbarBuilder.BeginSection(TEXT("Build"));
		{
			struct Local
			{
				static void ExecuteCompile(TSharedPtr<FCodeProjectEditor> InCodeProjectEditorPtr)
				{
					if(InCodeProjectEditorPtr->SaveAll())
					{
						FLevelEditorActionCallbacks::RecompileGameCode_Clicked();
					}
				}
			};

			// Since we can always add new code to the project, only hide these buttons if we haven't done so yet
			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateStatic(&Local::ExecuteCompile, CodeProjectEditorPtr),
					FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Recompile_CanExecute),
					FIsActionChecked(),
					FIsActionButtonVisible::CreateStatic(FLevelEditorActionCallbacks::CanShowSourceCodeActions)),
				NAME_None,
				NSLOCTEXT( "LevelEditorToolBar", "CompileMenuButton", "Compile" ),
				NSLOCTEXT( "LevelEditorActions", "RecompileGameCode_ToolTip", "Recompiles and reloads C++ code for game systems on the fly" ),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile")
				);
		}
		ToolbarBuilder.EndSection();
	}
}
