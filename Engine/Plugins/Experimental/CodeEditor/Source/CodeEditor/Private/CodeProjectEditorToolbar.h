// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CodeProjectEditor.h"

class FToolBarBuilder;

class FCodeProjectEditorToolbar : public TSharedFromThis<FCodeProjectEditorToolbar>
{
public:
	FCodeProjectEditorToolbar(TSharedPtr<class FCodeProjectEditor> InCodeProjectEditor)
		: CodeProjectEditor(InCodeProjectEditor) {}

	void AddEditorToolbar(TSharedPtr<FExtender> Extender);

private:
	void FillEditorToolbar(FToolBarBuilder& ToolbarBuilder);

protected:
	/** Pointer back to the code editor tool that owns us */
	TWeakPtr<class FCodeProjectEditor> CodeProjectEditor;
};
