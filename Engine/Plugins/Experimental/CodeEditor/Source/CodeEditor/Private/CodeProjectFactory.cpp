// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeProjectFactory.h"
#include "CodeProject.h"


#define LOCTEXT_NAMESPACE "CodeEditor"


UCodeProjectFactory::UCodeProjectFactory(const class FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCodeProject::StaticClass();
}


UObject* UCodeProjectFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCodeProject* NewCodeProject = NewObject<UCodeProject>(InParent, Class, Name, Flags);
	return NewCodeProject;
}


#undef LOCTEXT_NAMESPACE
