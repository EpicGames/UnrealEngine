// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeProjectItem.h"
#include "UObject/Package.h"
#include "IDirectoryWatcher.h"
#include "DirectoryScanner.h"

UCodeProjectItem::UCodeProjectItem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UCodeProjectItem::RescanChildren()
{
	if(Path.Len() > 0)
	{
		FDirectoryScanner::AddDirectory(Path, FOnDirectoryScanned::CreateUObject(this, &UCodeProjectItem::HandleDirectoryScanned));
	}
}

void UCodeProjectItem::HandleDirectoryScanned(const FString& InPathName, ECodeProjectItemType::Type InType)
{
	// check for a child that already exists
	bool bCreateNew = true;
	for(const auto& Child : Children)
	{
		if(Child->Type == InType && Child->Path == InPathName)
		{
			bCreateNew = false;
			break;
		}
	}

	// create children now & kick off their scan
	if(bCreateNew)
	{
		UCodeProjectItem* NewItem = NewObject<UCodeProjectItem>(GetOutermost(), UCodeProjectItem::StaticClass());
		NewItem->Type = InType;
		NewItem->Path = InPathName;
		NewItem->Name = FPaths::GetCleanFilename(InPathName);
		if(InType != ECodeProjectItemType::Folder)
		{
			NewItem->Extension = FPaths::GetExtension(InPathName);
		}

		Children.Add(NewItem);

		Children.Sort(
			[](const UCodeProjectItem& ItemA, const UCodeProjectItem& ItemB) -> bool
			{
				if(ItemA.Type != ItemB.Type)
				{
					return ItemA.Type < ItemB.Type;
				}

				return ItemA.Name.Compare(ItemB.Name) < 0;
			}
		);

		if(InType == ECodeProjectItemType::Folder)
		{
			// kick off another scan for subdirectories
			FDirectoryScanner::AddDirectory(InPathName, FOnDirectoryScanned::CreateUObject(NewItem, &UCodeProjectItem::HandleDirectoryScanned));

			// @TODO: now register for any changes to this directory if needed
		//	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
		//	DirectoryWatcherModule.Get()->RegisterDirectoryChangedCallback_Handle(InPathName, IDirectoryWatcher::FDirectoryChanged::CreateUObject(NewItem, &UCodeProjectItem::HandleDirectoryChanged), OnDirectoryChangedHandle);
		}
	}
}

void UCodeProjectItem::HandleDirectoryChanged(const TArray<FFileChangeData>& FileChanges)
{
	// @TODO: dynamical update directory watchers so we can update the view in real-time
	for(const auto& Change : FileChanges)
	{
		switch(Change.Action)
		{
		default:
		case FFileChangeData::FCA_Unknown:
			break;
		case FFileChangeData::FCA_Added:
			{

			}
			break;
		case FFileChangeData::FCA_Modified:
			{

			}
			break;
		case FFileChangeData::FCA_Removed:
			{

			}
			break;
		}
	}
}
