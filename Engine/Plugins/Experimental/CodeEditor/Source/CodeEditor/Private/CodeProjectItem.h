// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CodeProjectItem.generated.h"

/** Types of project items. Note that the enum ordering determines the tree sorting */
UENUM()
namespace ECodeProjectItemType
{
	enum Type : int
	{
		Project,
		Folder,
		File
	};
}

UCLASS()
class UCodeProjectItem : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	void RescanChildren();

	void HandleDirectoryScanned(const FString& InPathName, ECodeProjectItemType::Type InType);

	/** Handle directory changing */
	void HandleDirectoryChanged(const TArray<struct FFileChangeData>& FileChanges);

public:
	UPROPERTY(Transient)
	TEnumAsByte<ECodeProjectItemType::Type> Type;

	UPROPERTY(Transient)
	FString Name;

	UPROPERTY(Transient)
	FString Extension;

	UPROPERTY(Transient)
	FString Path;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UCodeProjectItem>> Children;

	/** Delegate handle for directory watcher */
	FDelegateHandle OnDirectoryChangedHandle;
};
