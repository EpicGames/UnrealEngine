// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryScanner.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/IQueuedWork.h"
#include "Misc/QueuedThreadPool.h"

TArray<FDirectoryScannerCommand*> FDirectoryScanner::CommandQueue;

struct FDirectoryResult
{
	FDirectoryResult(const FString& InPathName, ECodeProjectItemType::Type InType)
		: PathName(InPathName)
		, Type(InType)
	{
	}

	FString PathName;

	ECodeProjectItemType::Type Type;
};

struct FDirectoryScannerCommand : public IQueuedWork
{
	FDirectoryScannerCommand(const FString& InPathName, const FOnDirectoryScanned& InOnDirectoryScanned)
		: PathName(InPathName)
		, OnDirectoryScanned(InOnDirectoryScanned)
		, bExecuted(0)
	{
	}

	/** Begin FQueuedWork interface */
	virtual void Abandon() override
	{
		FPlatformAtomics::InterlockedExchange(&bExecuted, 1);
	}

	virtual void DoThreadedWork() override
	{
		class FDirectoryEnumerator : public IPlatformFile::FDirectoryVisitor
		{
		public:
			FDirectoryEnumerator(TLockFreePointerListUnordered<FDirectoryResult, PLATFORM_CACHE_LINE_SIZE>& InFoundFiles)
				: FoundFiles(InFoundFiles)
			{
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if(bIsDirectory)
				{
					FoundFiles.Push(new FDirectoryResult(FilenameOrDirectory, ECodeProjectItemType::Folder));
				}
				else
				{
					FoundFiles.Push(new FDirectoryResult(FilenameOrDirectory, ECodeProjectItemType::File));
				}

				return true;
			}

			TLockFreePointerListUnordered<FDirectoryResult, PLATFORM_CACHE_LINE_SIZE>& FoundFiles;
		};

		FDirectoryEnumerator DirectoryEnumerator(FoundFiles);
		IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
		PlatformFile.IterateDirectory(*PathName, DirectoryEnumerator);

		FPlatformAtomics::InterlockedExchange(&bExecuted, 1);
	}
	/** End FQueuedWork interface */

	FString PathName;

	FOnDirectoryScanned OnDirectoryScanned;

	TLockFreePointerListUnordered<FDirectoryResult, PLATFORM_CACHE_LINE_SIZE> FoundFiles;

	volatile int32 bExecuted;
};


bool FDirectoryScanner::Tick()
{
	bool bAddedData = false;
	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FDirectoryScannerCommand& Command = *CommandQueue[CommandIndex];
		if (Command.bExecuted)
		{
			while(!Command.FoundFiles.IsEmpty())
			{
				FDirectoryResult* DirectoryResult = Command.FoundFiles.Pop();
				Command.OnDirectoryScanned.ExecuteIfBound(DirectoryResult->PathName, DirectoryResult->Type);
				delete DirectoryResult;
				bAddedData = true;
			}

			// Remove command from the queue & delete it, we are done for this tick
			CommandQueue.RemoveAt(CommandIndex);
			delete &Command;
			break;
		}
	}

	return bAddedData;
}

void FDirectoryScanner::AddDirectory(const FString& PathName, const FOnDirectoryScanned& OnDirectoryScanned)
{
	FDirectoryScannerCommand* NewCommand = new FDirectoryScannerCommand(PathName, OnDirectoryScanned);
	CommandQueue.Add(NewCommand);
	GThreadPool->AddQueuedWork(NewCommand);
}

bool FDirectoryScanner::IsScanning()
{
	return CommandQueue.Num() > 0;
}
