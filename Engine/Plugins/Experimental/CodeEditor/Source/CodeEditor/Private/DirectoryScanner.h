// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CodeProjectItem.h"

DECLARE_DELEGATE_TwoParams(FOnDirectoryScanned, const FString& /*InPathName*/, ECodeProjectItemType::Type /*InType*/);

class FDirectoryScanner
{
public:
	static bool Tick();

	static void AddDirectory(const FString& PathName, const FOnDirectoryScanned& OnDirectoryScanned);

	static bool IsScanning() ;

public:
	static TArray<struct FDirectoryScannerCommand*> CommandQueue;

	static bool bDataDirty;
};
