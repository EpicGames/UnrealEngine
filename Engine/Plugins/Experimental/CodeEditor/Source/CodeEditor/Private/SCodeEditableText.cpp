// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCodeEditableText.h"
#include "CodeEditorStyle.h"


void SCodeEditableText::Construct( const FArguments& InArgs )
{
	SMultiLineEditableText::Construct(
		SMultiLineEditableText::FArguments()
		.Font(FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TextEditor.NormalText").Font)
		.TextStyle(&FCodeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TextEditor.NormalText"))
		.Text(InArgs._Text)
		.Marshaller(InArgs._Marshaller)
		.AutoWrapText(false)
		.Margin(0.0f)
		.HScrollBar(InArgs._HScrollBar)
		.VScrollBar(InArgs._VScrollBar)
		.OnTextChanged(InArgs._OnTextChanged)
	);
}

FReply SCodeEditableText::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	FReply Reply = FReply::Unhandled();

	const TCHAR Character = InCharacterEvent.GetCharacter();
	if(Character == TEXT('\t'))
	{
		if (!IsTextReadOnly())
		{
			FString String;
			String.AppendChar(Character);
			InsertTextAtCursor(String);
			Reply = FReply::Handled();
		}
		else
		{
			Reply = FReply::Unhandled();
		}
	}
	else
	{
		Reply = SMultiLineEditableText::OnKeyChar( MyGeometry, InCharacterEvent );
	}

	return Reply;
}
