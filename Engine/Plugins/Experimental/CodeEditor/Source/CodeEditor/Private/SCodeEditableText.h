// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Text/SMultiLineEditableText.h"

class SScrollBar;

class ITextLayoutMarshaller;

class SCodeEditableText : public SMultiLineEditableText
{
	SLATE_BEGIN_ARGS( SCodeEditableText )
	{}
		/** The initial text that will appear in the widget. */
		SLATE_ATTRIBUTE(FText, Text)

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr< ITextLayoutMarshaller >, Marshaller)

		/** The horizontal scroll bar widget */
		SLATE_ARGUMENT(TSharedPtr< SScrollBar >, HScrollBar)

		/** The vertical scroll bar widget */
		SLATE_ARGUMENT(TSharedPtr< SScrollBar >, VScrollBar)

		/** Called whenever the text is changed interactively by the user */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

private:
	virtual FReply OnKeyChar(const FGeometry& MyGeometry,const FCharacterEvent& InCharacterEvent) override;
};
