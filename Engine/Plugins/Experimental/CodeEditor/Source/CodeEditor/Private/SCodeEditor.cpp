// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCodeEditor.h"
#include "Misc/FileHelper.h"
#include "Widgets/Layout/SGridPanel.h"
#include "CodeProjectItem.h"
#include "CPPRichTextSyntaxHighlighterTextLayoutMarshaller.h"
#include "SCodeEditableText.h"


#define LOCTEXT_NAMESPACE "CodeEditor"


void SCodeEditor::Construct(const FArguments& InArgs, UCodeProjectItem* InCodeProjectItem)
{
	bDirty = false;

	check(InCodeProjectItem);
	CodeProjectItem = InCodeProjectItem;

	FString FileText = "File Loading, please wait";
	FFileHelper::LoadFileToString(FileText, *InCodeProjectItem->Path);

	TSharedRef<FCPPRichTextSyntaxHighlighterTextLayoutMarshaller> RichTextMarshaller = FCPPRichTextSyntaxHighlighterTextLayoutMarshaller::Create(
			FCPPRichTextSyntaxHighlighterTextLayoutMarshaller::FSyntaxTextStyle()
			);

	HorizontalScrollbar = 
		SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(14.0f, 14.0f));

	VerticalScrollbar = 
		SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(14.0f, 14.0f));

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCodeEditorStyle::Get().GetBrush("TextEditor.Border"))
		[
			SNew(SGridPanel)
			.FillColumn(0, 1.0f)
			.FillRow(0, 1.0f)
			+SGridPanel::Slot(0, 0)
			[
				SAssignNew(CodeEditableText, SCodeEditableText)
				.Text(FText::FromString(FileText))
				.Marshaller(RichTextMarshaller)
				.HScrollBar(HorizontalScrollbar)
				.VScrollBar(VerticalScrollbar)
				.OnTextChanged(this, &SCodeEditor::OnTextChanged)
			]
			+SGridPanel::Slot(1, 0)
			[
				VerticalScrollbar.ToSharedRef()
			]
			+SGridPanel::Slot(0, 1)
			[
				HorizontalScrollbar.ToSharedRef()
			]
		]
	];
}

void SCodeEditor::OnTextChanged(const FText& NewText)
{
	bDirty = true;
}

bool SCodeEditor::Save() const
{
	if(bDirty)
	{
		bool bResult = FFileHelper::SaveStringToFile(CodeEditableText->GetText().ToString(), *CodeProjectItem->Path);
		if(bResult)
		{
			bDirty = false;
		}

		return bResult;
	}
	return true;
}

bool SCodeEditor::CanSave() const
{
	return bDirty;
}

void SCodeEditor::GotoLineAndColumn(int32 LineNumber, int32 ColumnNumber)
{
	FTextLocation Location(LineNumber, ColumnNumber);
	CodeEditableText->GoTo(Location);
	CodeEditableText->ScrollTo(Location);
}

#undef LOCTEXT_NAMESPACE
