// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SCodeEditableText;
class SScrollBar;

class SCodeEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCodeEditor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class UCodeProjectItem* InCodeProjectItem);

	bool Save() const;

	bool CanSave() const;

	void GotoLineAndColumn(int32 LineNumber, int32 ColumnNumber);

private:
	void OnTextChanged(const FText& NewText);

protected:
	class UCodeProjectItem* CodeProjectItem;

	TSharedPtr<SScrollBar> HorizontalScrollbar;
	TSharedPtr<SScrollBar> VerticalScrollbar;

	TSharedPtr<class SCodeEditableText> CodeEditableText;

	mutable bool bDirty;
};
