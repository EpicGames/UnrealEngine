// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCodeProjectEditor.h"
#include "CodeEditorStyle.h"
#include "CodeProject.h"
#include "CodeProjectEditor.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SProjectViewItem.h"
#include "DirectoryScanner.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Views/STreeView.h"


#define LOCTEXT_NAMESPACE "CodeProjectEditor"


void SCodeProjectEditor::Construct(const FArguments& InArgs, UCodeProject* InCodeProject)
{
	check(InCodeProject);
	CodeProject = InCodeProject;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCodeEditorStyle::Get().GetBrush("ProjectEditor.Border"))
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SAssignNew(ProjectTree, STreeView<UCodeProjectItem*>)
				.TreeItemsSource(&ToRawPtrTArrayUnsafe(CodeProject->Children))
				.OnGenerateRow(this, &SCodeProjectEditor::OnGenerateRow)
				.OnGetChildren(this, &SCodeProjectEditor::OnGetChildren)
				.OnMouseButtonDoubleClick(this, &SCodeProjectEditor::HandleMouseButtonDoubleClick)
			]
			+SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			.Padding(10.0f)
			[
				SNew(SThrobber)
				.Visibility(this, &SCodeProjectEditor::GetThrobberVisibility)
			]
		]
	];

	InCodeProject->RescanChildren();
}

void SCodeProjectEditor::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if(FDirectoryScanner::Tick())
	{
		ProjectTree->SetTreeItemsSource(&ToRawPtrTArrayUnsafe(CodeProject->Children));
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

FName SCodeProjectEditor::GetIconForItem(UCodeProjectItem* Item) const
{
	switch(Item->Type)
	{
	case ECodeProjectItemType::Project:
		return "ProjectEditor.Icon.Project";
	case ECodeProjectItemType::Folder:
		return "ProjectEditor.Icon.Folder";
	case ECodeProjectItemType::File:
		return "ProjectEditor.Icon.File";
	default:
		return "ProjectEditor.Icon.GenericFile";
	}
}

TSharedRef<class ITableRow> SCodeProjectEditor::OnGenerateRow(UCodeProjectItem* Item, const TSharedRef<STableViewBase >& OwnerTable)
{
	return
		SNew(STableRow<UCodeProjectItem*>, OwnerTable)
		[
			SNew(SProjectViewItem)
			.Text(FText::FromString(Item->Name))
			.IconName(GetIconForItem(Item))
		];
}

void SCodeProjectEditor::OnGetChildren(UCodeProjectItem* Item, TArray<UCodeProjectItem*>& OutChildItems)
{
	OutChildItems = Item->Children;
}

EVisibility SCodeProjectEditor::GetThrobberVisibility() const
{ 
	return FDirectoryScanner::IsScanning() ? EVisibility::Visible : EVisibility::Hidden; 
}

void SCodeProjectEditor::HandleMouseButtonDoubleClick(UCodeProjectItem* Item) const
{
	if(Item->Type == ECodeProjectItemType::File)
	{
		FCodeProjectEditor::Get()->OpenFileForEditing(Item);
	}
}

#undef LOCTEXT_NAMESPACE
