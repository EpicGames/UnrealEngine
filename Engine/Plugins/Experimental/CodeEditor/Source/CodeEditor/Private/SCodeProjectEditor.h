// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

template <typename ItemType> class STreeView;

class SCodeProjectEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCodeProjectEditor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class UCodeProject* InCodeProject);

private:
	/** Begin SWidget interface */
	void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	/** End SWidget interface */

	TSharedRef<class ITableRow> OnGenerateRow(class UCodeProjectItem* Item, const TSharedRef<class STableViewBase>& TableView);

	void OnGetChildren(class UCodeProjectItem* Item, TArray<class UCodeProjectItem*>& OutChildItems);

	EVisibility GetThrobberVisibility() const;

	FName GetIconForItem(class UCodeProjectItem* Item) const;

	void HandleMouseButtonDoubleClick(class UCodeProjectItem* Item) const;

private:
	class UCodeProject* CodeProject;

	TSharedPtr<STreeView<class UCodeProjectItem*>> ProjectTree;
};
