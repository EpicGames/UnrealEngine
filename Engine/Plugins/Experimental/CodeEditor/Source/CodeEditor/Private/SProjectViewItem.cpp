// Copyright Epic Games, Inc. All Rights Reserved.

#include "SProjectViewItem.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "CodeEditorStyle.h"


#define LOCTEXT_NAMESPACE "ProjectViewItem"


void SProjectViewItem::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(1.0f)
		.AutoWidth()
		[
			SNew(SImage)
			.Image(FCodeEditorStyle::Get().GetBrush(InArgs._IconName))
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(1.0f)
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(InArgs._Text)
		]
	];
}


#undef LOCTEXT_NAMESPACE
