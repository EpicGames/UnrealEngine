// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SProjectViewItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SProjectViewItem) {}

	SLATE_ARGUMENT(FName, IconName)

	SLATE_ARGUMENT(FText, Text)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
