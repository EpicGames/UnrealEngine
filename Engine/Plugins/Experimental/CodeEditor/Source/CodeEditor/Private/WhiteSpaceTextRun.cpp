// Copyright Epic Games, Inc. All Rights Reserved.

#include "WhiteSpaceTextRun.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"

TSharedRef< FWhiteSpaceTextRun > FWhiteSpaceTextRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style, const FTextRange& InRange, int32 NumSpacesPerTab )
{
	return MakeShareable( new FWhiteSpaceTextRun( InRunInfo, InText, Style, InRange, NumSpacesPerTab ) );
}

FVector2D FWhiteSpaceTextRun::Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext) const
{
	const FVector2D ShadowOffsetToApply((EndIndex == Range.EndIndex) ? FMath::Abs(Style.ShadowOffset.X * Scale) : 0.0f, FMath::Abs(Style.ShadowOffset.Y * Scale));

	if ( EndIndex - StartIndex == 0 )
	{
		return FVector2D( ShadowOffsetToApply.X * Scale, GetMaxHeight( Scale ) );
	}

	// count tabs
	int32 TabCount = 0;
	for(int32 Index = StartIndex; Index < EndIndex; Index++)
	{
		if((*Text)[Index] == TEXT('\t'))
		{
			TabCount++;
		}
	}

	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FVector2D Size = FontMeasure->Measure( **Text, StartIndex, EndIndex, Style.Font, true, Scale ) + ShadowOffsetToApply;

	Size.X -= TabWidth * (float)TabCount * Scale;
	Size.X += SpaceWidth * (float)(TabCount * NumSpacesPerTab) * Scale;

	return Size;
}

FWhiteSpaceTextRun::FWhiteSpaceTextRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle, const FTextRange& InRange, int32 InNumSpacesPerTab ) 
	: FSlateTextRun(InRunInfo, InText, InStyle, InRange)
	, NumSpacesPerTab(InNumSpacesPerTab)
{
	// measure tab width
	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	TabWidth = FontMeasure->Measure( TEXT("\t"), 0, 1, Style.Font, true, 1.0f ).X;
	SpaceWidth = FontMeasure->Measure( TEXT(" "), 0, 1, Style.Font, true, 1.0f ).X;
}

