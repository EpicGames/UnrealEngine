// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Text/SlateTextRun.h"

struct FRunInfo;

class FWhiteSpaceTextRun : public FSlateTextRun
{
public:
	static TSharedRef< FWhiteSpaceTextRun > Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style, const FTextRange& InRange, int32 NumSpacesPerTab );

public:
	virtual FVector2D Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext) const override;

protected:
	FWhiteSpaceTextRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle, const FTextRange& InRange, int32 InNumSpacesPerTab );

private:
	int32 NumSpacesPerTab;

	float TabWidth;

	float SpaceWidth;
};
